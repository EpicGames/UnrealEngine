// Copyright Epic Games, Inc. All Rights Reserved.

#include "TextureBuildFunction.h"

#include "ChildTextureFormat.h"
#include "DerivedDataCache.h"
#include "DerivedDataValueId.h"
#include "Engine/TextureDefines.h"
#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "ImageCore.h"
#include "ImageCoreUtils.h"
#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatModule.h"
#include "IO/IoHash.h"
#include "Memory/CompositeBuffer.h"
#include "Memory/SharedBuffer.h"
#include "Modules/ModuleManager.h"
#include "PixelFormat.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/FileRegions.h"
#include "Serialization/MemoryWriter.h"
#include "TextureBuildUtilities.h"
#include "TextureCompressorModule.h"
#include "TextureFormatManager.h"
#include "Misc/CommandLine.h"
#include "HAL/FileManager.h"

DEFINE_LOG_CATEGORY_STATIC(LogTextureBuildFunction, Log, All);

// Any edits to the texture compressor or this file that will change the output of texture builds
// MUST have a corresponding change to this version. Individual texture formats have a version to
// change that is specific to the format. A merge conflict affecting the version MUST be resolved
// by generating a new version. This also includes the addition of new outputs to the build, as
// this will cause a DDC verification failure unless a new version is created.
// A reminder that for DDC invalidation, running a ddc fill job or the ddc commandlet is a friendly
// thing to do! -run=DerivedDataCache -Fill -TargetPlatform=Platform1,Platform...N
//
static const FGuid TextureBuildFunctionVersion(TEXT("B20676CE-A786-43EE-96F0-2620A4C38ACA"));

static void ReadCbField(FCbFieldView Field, bool& OutValue) { OutValue = Field.AsBool(OutValue); }
static void ReadCbField(FCbFieldView Field, int32& OutValue) { OutValue = Field.AsInt32(OutValue); }
static void ReadCbField(FCbFieldView Field, uint8& OutValue) { OutValue = Field.AsUInt8(OutValue); }
static void ReadCbField(FCbFieldView Field, uint32& OutValue) { OutValue = Field.AsUInt32(OutValue); }
static void ReadCbField(FCbFieldView Field, float& OutValue) { OutValue = Field.AsFloat(OutValue); }
static void ReadCbField(FCbFieldView Field, FGuid& OutValue) { OutValue = Field.AsUuid(); }

static void ReadCbField(FCbFieldView Field, FName& OutValue)
{
	if (Field.IsString())
	{
		OutValue = FName(FUTF8ToTCHAR(Field.AsString()));
	}
}

static void ReadCbField(FCbFieldView Field, FColor& OutValue)
{
	FCbFieldViewIterator It = Field.AsArrayView().CreateViewIterator();
	OutValue.A = It++->AsUInt8(OutValue.A);
	OutValue.R = It++->AsUInt8(OutValue.R);
	OutValue.G = It++->AsUInt8(OutValue.G);
	OutValue.B = It++->AsUInt8(OutValue.B);
}

static void ReadCbField(FCbFieldView Field, FVector2f& OutValue)
{
	FCbFieldViewIterator It = Field.AsArrayView().CreateViewIterator();
	OutValue.X = It++->AsFloat(OutValue.X);
	OutValue.Y = It++->AsFloat(OutValue.Y);
}

static void ReadCbField(FCbFieldView Field, FVector4f& OutValue)
{
	FCbFieldViewIterator It = Field.AsArrayView().CreateViewIterator();
	OutValue.X = It++->AsFloat(OutValue.X);
	OutValue.Y = It++->AsFloat(OutValue.Y);
	OutValue.Z = It++->AsFloat(OutValue.Z);
	OutValue.W = It++->AsFloat(OutValue.W);
}

static void ReadCbField(FCbFieldView Field, FIntPoint& OutValue)
{
	FCbFieldViewIterator It = Field.AsArrayView().CreateViewIterator();
	OutValue.X = It++->AsInt32(OutValue.X);
	OutValue.Y = It++->AsInt32(OutValue.Y);
}

static FTextureBuildSettings ReadBuildSettingsFromCompactBinary(const FCbObjectView& Object)
{
	FTextureBuildSettings BuildSettings;
	BuildSettings.FormatConfigOverride = Object["FormatConfigOverride"].AsObjectView();
	FCbObjectView ColorAdjustmentCbObj = Object["ColorAdjustment"].AsObjectView();
	FColorAdjustmentParameters& ColorAdjustment = BuildSettings.ColorAdjustment;
	ReadCbField(ColorAdjustmentCbObj["AdjustBrightness"], ColorAdjustment.AdjustBrightness);
	ReadCbField(ColorAdjustmentCbObj["AdjustBrightnessCurve"], ColorAdjustment.AdjustBrightnessCurve);
	ReadCbField(ColorAdjustmentCbObj["AdjustSaturation"], ColorAdjustment.AdjustSaturation);
	ReadCbField(ColorAdjustmentCbObj["AdjustVibrance"], ColorAdjustment.AdjustVibrance);
	ReadCbField(ColorAdjustmentCbObj["AdjustRGBCurve"], ColorAdjustment.AdjustRGBCurve);
	ReadCbField(ColorAdjustmentCbObj["AdjustHue"], ColorAdjustment.AdjustHue);
	ReadCbField(ColorAdjustmentCbObj["AdjustMinAlpha"], ColorAdjustment.AdjustMinAlpha);
	ReadCbField(ColorAdjustmentCbObj["AdjustMaxAlpha"], ColorAdjustment.AdjustMaxAlpha);
	BuildSettings.bUseNewMipFilter = Object["bUseNewMipFilter"].AsBool(BuildSettings.bUseNewMipFilter);
	BuildSettings.bNormalizeNormals = Object["bNormalizeNormals"].AsBool(BuildSettings.bNormalizeNormals);
	BuildSettings.bDoScaleMipsForAlphaCoverage = Object["bDoScaleMipsForAlphaCoverage"].AsBool(BuildSettings.bDoScaleMipsForAlphaCoverage);
	ReadCbField(Object["AlphaCoverageThresholds"], BuildSettings.AlphaCoverageThresholds);
	ReadCbField(Object["MipSharpening"], BuildSettings.MipSharpening);
	ReadCbField(Object["DiffuseConvolveMipLevel"], BuildSettings.DiffuseConvolveMipLevel);
	ReadCbField(Object["SharpenMipKernelSize"], BuildSettings.SharpenMipKernelSize);
	ReadCbField(Object["MaxTextureResolution"], BuildSettings.MaxTextureResolution);
	check( BuildSettings.MaxTextureResolution != 0 );
	ReadCbField(Object["TextureFormatName"], BuildSettings.TextureFormatName);
	ReadCbField(Object["bHDRSource"], BuildSettings.bHDRSource);
	ReadCbField(Object["MipGenSettings"], BuildSettings.MipGenSettings);
	BuildSettings.bCubemap = Object["bCubemap"].AsBool(BuildSettings.bCubemap);
	BuildSettings.bTextureArray = Object["bTextureArray"].AsBool(BuildSettings.bTextureArray);
	BuildSettings.bVolume = Object["bVolume"].AsBool(BuildSettings.bVolume);
	BuildSettings.bLongLatSource = Object["bLongLatSource"].AsBool(BuildSettings.bLongLatSource);
	BuildSettings.bSRGB = Object["bSRGB"].AsBool(BuildSettings.bSRGB);
	ReadCbField(Object["SourceEncodingOverride"], BuildSettings.SourceEncodingOverride);
	BuildSettings.bHasColorSpaceDefinition = Object["bHasColorSpaceDefinition"].AsBool(BuildSettings.bHasColorSpaceDefinition);
	ReadCbField(Object["RedChromaticityCoordinate"], BuildSettings.RedChromaticityCoordinate);
	ReadCbField(Object["GreenChromaticityCoordinate"], BuildSettings.GreenChromaticityCoordinate);
	ReadCbField(Object["BlueChromaticityCoordinate"], BuildSettings.BlueChromaticityCoordinate);
	ReadCbField(Object["WhiteChromaticityCoordinate"], BuildSettings.WhiteChromaticityCoordinate);
	ReadCbField(Object["ChromaticAdaptationMethod"], BuildSettings.ChromaticAdaptationMethod);
	BuildSettings.bUseLegacyGamma = Object["bUseLegacyGamma"].AsBool(BuildSettings.bUseLegacyGamma);
	BuildSettings.bPreserveBorder = Object["bPreserveBorder"].AsBool(BuildSettings.bPreserveBorder);
	BuildSettings.bForceNoAlphaChannel = Object["bForceNoAlphaChannel"].AsBool(BuildSettings.bForceNoAlphaChannel);
	BuildSettings.bForceAlphaChannel = Object["bForceAlphaChannel"].AsBool(BuildSettings.bForceAlphaChannel);
	BuildSettings.bComputeBokehAlpha = Object["bComputeBokehAlpha"].AsBool(BuildSettings.bComputeBokehAlpha);
	BuildSettings.bReplicateRed = Object["bReplicateRed"].AsBool(BuildSettings.bReplicateRed);
	BuildSettings.bReplicateAlpha = Object["bReplicateAlpha"].AsBool(BuildSettings.bReplicateAlpha);
	BuildSettings.bDownsampleWithAverage = Object["bDownsampleWithAverage"].AsBool(BuildSettings.bDownsampleWithAverage);
	BuildSettings.bSharpenWithoutColorShift = Object["bSharpenWithoutColorShift"].AsBool(BuildSettings.bSharpenWithoutColorShift);
	BuildSettings.bBorderColorBlack = Object["bBorderColorBlack"].AsBool(BuildSettings.bBorderColorBlack);
	BuildSettings.bFlipGreenChannel = Object["bFlipGreenChannel"].AsBool(BuildSettings.bFlipGreenChannel);
	BuildSettings.bApplyYCoCgBlockScale = Object["bApplyYCoCgBlockScale"].AsBool(BuildSettings.bApplyYCoCgBlockScale);
	BuildSettings.bApplyKernelToTopMip = Object["bApplyKernelToTopMip"].AsBool(BuildSettings.bApplyKernelToTopMip);
	BuildSettings.bRenormalizeTopMip = Object["bRenormalizeTopMip"].AsBool(BuildSettings.bRenormalizeTopMip);
	BuildSettings.bCPUAccessible = Object["bCPUAccessible"].AsBool(BuildSettings.bCPUAccessible);
	ReadCbField(Object["CompositeTextureMode"], BuildSettings.CompositeTextureMode);
	ReadCbField(Object["CompositePower"], BuildSettings.CompositePower);
	ReadCbField(Object["LODBias"], BuildSettings.LODBias);
	ReadCbField(Object["LODBiasWithCinematicMips"], BuildSettings.LODBiasWithCinematicMips);
	BuildSettings.bStreamable_Unused = Object["bStreamable"].AsBool(BuildSettings.bStreamable_Unused);
	BuildSettings.bVirtualStreamable = Object["bVirtualStreamable"].AsBool(BuildSettings.bVirtualStreamable);
	BuildSettings.bChromaKeyTexture = Object["bChromaKeyTexture"].AsBool(BuildSettings.bChromaKeyTexture);
	ReadCbField(Object["PowerOfTwoMode"], BuildSettings.PowerOfTwoMode);
	ReadCbField(Object["PaddingColor"], BuildSettings.PaddingColor);
	BuildSettings.bPadWithBorderColor = Object["bPadWithBorderColor"].AsBool(BuildSettings.bPadWithBorderColor);
	ReadCbField(Object["ResizeDuringBuildX"], BuildSettings.ResizeDuringBuildX);
	ReadCbField(Object["ResizeDuringBuildY"], BuildSettings.ResizeDuringBuildY);
	ReadCbField(Object["ChromaKeyColor"], BuildSettings.ChromaKeyColor);
	ReadCbField(Object["ChromaKeyThreshold"], BuildSettings.ChromaKeyThreshold);
	ReadCbField(Object["CompressionQuality"], BuildSettings.CompressionQuality);
	ReadCbField(Object["LossyCompressionAmount"], BuildSettings.LossyCompressionAmount);
	ReadCbField(Object["Downscale"], BuildSettings.Downscale);
	ReadCbField(Object["DownscaleOptions"], BuildSettings.DownscaleOptions);
	ReadCbField(Object["VirtualAddressingModeX"], BuildSettings.VirtualAddressingModeX);
	ReadCbField(Object["VirtualAddressingModeY"], BuildSettings.VirtualAddressingModeY);
	ReadCbField(Object["VirtualTextureTileSize"], BuildSettings.VirtualTextureTileSize);
	ReadCbField(Object["VirtualTextureBorderSize"], BuildSettings.VirtualTextureBorderSize);
	BuildSettings.OodleEncodeEffort = Object["OodleEncodeEffort"].AsUInt8(BuildSettings.OodleEncodeEffort);
	BuildSettings.OodleUniversalTiling = Object["OodleUniversalTiling"].AsUInt8(BuildSettings.OodleUniversalTiling);
	BuildSettings.bOodleUsesRDO = Object["bOodleUsesRDO"].AsBool(BuildSettings.bOodleUsesRDO);
	BuildSettings.OodleRDO = Object["OodleRDO"].AsUInt8(BuildSettings.OodleRDO);
	BuildSettings.bOodlePreserveExtremes = Object["bOodlePreserveExtremes"].AsBool(BuildSettings.bOodlePreserveExtremes);
	ReadCbField(Object["OodleTextureSdkVersion"], BuildSettings.OodleTextureSdkVersion);
	ReadCbField(Object["TextureAddressModeX"], BuildSettings.TextureAddressModeX);
	ReadCbField(Object["TextureAddressModeY"], BuildSettings.TextureAddressModeY);
	ReadCbField(Object["TextureAddressModeZ"], BuildSettings.TextureAddressModeZ);

	return BuildSettings;
}


static bool GetResolvedBuildSettings(const FCbObject& Settings, FTextureBuildSettings* OutBuildSettings, const ITextureFormat** OutTextureFormat)
{
	*OutBuildSettings = ReadBuildSettingsFromCompactBinary(Settings["Build"].AsObjectView());

	const uint16 RequiredTextureFormatVersion = Settings["FormatVersion"].AsUInt16();
	const ITextureFormat* TextureFormat = nullptr;
	if (ITextureFormatManagerModule* TFM = GetTextureFormatManager())
	{
		TextureFormat = TFM->FindTextureFormat(OutBuildSettings->TextureFormatName);
	}
	else
	{
		UE_LOGF(LogTextureBuildFunction, Error, "TextureFormatManager not found!");
		return false;
	}

	if (!TextureFormat)
	{
		UE_LOGF(LogTextureBuildFunction, Error, "Texture format %ls not found", *WriteToString<128>(OutBuildSettings->TextureFormatName));
		return false;
	}

	if (OutTextureFormat)
	{
		*OutTextureFormat = TextureFormat;
	}

	const uint16 CurrentTextureFormatVersion = TextureFormat->GetVersion(OutBuildSettings->TextureFormatName, OutBuildSettings);
	if (CurrentTextureFormatVersion != RequiredTextureFormatVersion)
	{
		UE_LOGF(LogTextureBuildFunction, Error, "%ls has version %hu when version %hu is required.",
			*OutBuildSettings->TextureFormatName.ToString(), CurrentTextureFormatVersion, RequiredTextureFormatVersion);;
		return false;
	}

	const FChildTextureFormat* ChildTextureFormat = TextureFormat->GetChildFormat();
	if (ChildTextureFormat)
	{
		OutBuildSettings->BaseTextureFormatName = ChildTextureFormat->GetBaseFormatName(OutBuildSettings->TextureFormatName);
	}
	else
	{
		OutBuildSettings->BaseTextureFormatName = OutBuildSettings->TextureFormatName;
	}

	OutBuildSettings->BaseTextureFormat = GetTextureFormatManager()->FindTextureFormat(OutBuildSettings->BaseTextureFormatName);
	return true;
}


static bool GetImageInfoFromCb(FCbFieldView InSource, FImageInfo* OutImageInfo, int32* OutMipCount)
{
	OutImageInfo->Format = FImageCoreUtils::ConvertToRawImageFormat((ETextureSourceFormat)InSource["SourceFormat"].AsUInt8());
	OutImageInfo->GammaSpace = (EGammaSpace)InSource["GammaSpace"].AsUInt8();
	OutImageInfo->NumSlices = InSource["NumSlices"].AsInt32();
	OutImageInfo->SizeX = InSource["SizeX"].AsInt32();
	OutImageInfo->SizeY = InSource["SizeY"].AsInt32();

	*OutMipCount = InSource["Mips"].AsArrayView().Num();

	return true;
}


static bool TryReadTextureSourceFromCompactBinary(FCbFieldView Source, UE::DerivedData::FBuildContext& Context,
												bool bVolume, TArray<FImage>& OutMips)
{
	FSharedBuffer InputBuffer = Context.FindInput(Source.GetName());
	if (!InputBuffer)
	{
		UE_LOGF(LogTextureBuildFunction, Error, "Missing input '%ls'.", *WriteToString<64>(Source.GetName()));
		return false;
	}
	if ( InputBuffer.GetSize() == 0 )
	{
		UE_LOGF(LogTextureBuildFunction, Error, "Input size zero '%ls'.", *WriteToString<64>(Source.GetName()));
		return false;
	}

	// Source data has no CompressionFormat
	ETextureSourceFormat SourceFormat = (ETextureSourceFormat)Source["SourceFormat"].AsUInt8();

	ERawImageFormat::Type RawImageFormat = FImageCoreUtils::ConvertToRawImageFormat(SourceFormat);

	EGammaSpace GammaSpace = (EGammaSpace)Source["GammaSpace"].AsUInt8();
	int32 NumSlices = Source["NumSlices"].AsInt32();
	int32 SizeX = Source["SizeX"].AsInt32();
	int32 SizeY = Source["SizeY"].AsInt32();
	int32 MipSizeX = SizeX;
	int32 MipSizeY = SizeY;

	const uint8* DecompressedSourceData = (const uint8*)InputBuffer.GetData();
	int64 DecompressedSourceDataSize = InputBuffer.GetSize();

	FCbArrayView MipsCbArrayView = Source["Mips"].AsArrayView();
	OutMips.Reserve(IntCastChecked<int32>(MipsCbArrayView.Num()));
	for (FCbFieldView MipsCbArrayIt : MipsCbArrayView)
	{
		FCbObjectView MipCbObjectView = MipsCbArrayIt.AsObjectView();
		int64 MipOffset = MipCbObjectView["Offset"].AsInt64();
		int64 MipSize = MipCbObjectView["Size"].AsInt64();

		FImage& SourceMip = OutMips.Emplace_GetRef(
			MipSizeX, MipSizeY,
			NumSlices,
			RawImageFormat,
			GammaSpace
		);

		check( MipOffset + MipSize <= DecompressedSourceDataSize );
		check( SourceMip.GetImageSizeBytes() == MipSize );

		SourceMip.RawData.Reset(MipSize);
		SourceMip.RawData.AddUninitialized(MipSize);

		FMemory::Memcpy(
			SourceMip.RawData.GetData(),
			DecompressedSourceData + MipOffset,
			MipSize
		);

		MipSizeX = FEncodedTextureDescription::GetMipWidth(MipSizeX, 1);
		MipSizeY = FEncodedTextureDescription::GetMipHeight(MipSizeY, 1);
		if ( bVolume )
		{
			NumSlices = FEncodedTextureDescription::GetMipDepth(NumSlices, 1, true);
		}
	}

	return true;
}

FGuid FTextureBuildFunction::GetVersion() const
{
	UE::DerivedData::FBuildVersionBuilder Builder;
	Builder << TextureBuildFunctionVersion;
	ITextureFormat* TextureFormat = nullptr;
	GetVersion(Builder, TextureFormat);
	if (TextureFormat)
	{
		TArray<FName> SupportedFormats;
		TextureFormat->GetSupportedFormats(SupportedFormats);
		TArray<uint16> SupportedFormatVersions;
		for (const FName& SupportedFormat : SupportedFormats)
		{
			SupportedFormatVersions.AddUnique(TextureFormat->GetVersion(SupportedFormat));
		}
		SupportedFormatVersions.Sort();
		Builder << SupportedFormatVersions;
	}
	return Builder.Build();
}

void FTextureBuildFunction::Configure(UE::DerivedData::FBuildConfigContext& Context) const
{
	Context.SetTypeName(UTF8TEXTVIEW("Texture"));
	Context.SetCacheBucket(UE::DerivedData::FCacheBucket(ANSITEXTVIEW("Texture")));

	const FCbObject Settings = Context.FindConstant(UTF8TEXTVIEW("Settings"));

	// Bit unfortunate - we have to deserialize this entire thing in order to be able to compute
	// the memory estimate, and we're about the deserialize the whole things again.	
	FTextureBuildSettings BuildSettings;
	if (GetResolvedBuildSettings(Settings, &BuildSettings, nullptr))
	{	
		FImageInfo SourceImageInfo;
		int32 SourceMipCount = 0;
		if (GetImageInfoFromCb(Settings["Source"], &SourceImageInfo, &SourceMipCount))
		{
			const int64 RequiredMemoryEstimate = UE::TextureBuildUtilities::GetPhysicalTextureBuildMemoryEstimate(&BuildSettings, SourceImageInfo, SourceMipCount);
			Context.SetRequiredMemory(RequiredMemoryEstimate);
		}
	}
}

// All texture builds output (at least) these values.
struct FChildBuildData
{
	FEncodedTextureDescription TextureDescription;
	FEncodedTextureExtendedData TextureExtendedData;
	FTextureEngineParameters EngineParameters;
	FEncodedTextureDescription::FSharedBufferMipChain MipBuffers;

	// Cache these values since we always need them
	int32 NumStreamingMips, NumEncodedMips;

	// Pass-thru values.
	FCompositeBuffer CPUCopyImageInfo;
	FSharedBuffer CPUCopyRawData;
};

static bool ReadChildBuildInputs(FChildBuildData& OutChildBuildInputs, UE::DerivedData::FBuildContext& Context)
{
	{
		FSharedBuffer RawTextureDescription = Context.FindInput(UTF8TEXTVIEW("EncodedTextureDescription"));
		if (!RawTextureDescription)
		{
			Context.AddError(TEXTVIEW("Missing EncodedTextureDescription"));
			return false;
		}
		UE::TextureBuildUtilities::EncodedTextureDescription::FromCompactBinary(OutChildBuildInputs.TextureDescription, FCbObject(RawTextureDescription));
	}

	{
		FSharedBuffer RawTextureExtendedData = Context.FindInput(UTF8TEXTVIEW("EncodedTextureExtendedData"));
		if (!RawTextureExtendedData)
		{
			Context.AddError(TEXTVIEW("Missing EncodedTextureExtendedData"));
			return false;
		}
		UE::TextureBuildUtilities::EncodedTextureExtendedData::FromCompactBinary(OutChildBuildInputs.TextureExtendedData, FCbObject(RawTextureExtendedData));
	}

	{
		FCbObject EngineParametersCb = Context.FindConstant(UTF8TEXTVIEW("EngineParameters"));
		UE::TextureBuildUtilities::TextureEngineParameters::FromCompactBinary(OutChildBuildInputs.EngineParameters, EngineParametersCb);
	}

	OutChildBuildInputs.NumStreamingMips = OutChildBuildInputs.TextureDescription.GetNumStreamingMips(&OutChildBuildInputs.TextureExtendedData, OutChildBuildInputs.EngineParameters);
	OutChildBuildInputs.NumEncodedMips = OutChildBuildInputs.TextureDescription.GetNumEncodedMips(&OutChildBuildInputs.TextureExtendedData);

	// Extended data mip sizes should always be valid with either linear mip sizes or tiled mip sizes.
	check(OutChildBuildInputs.TextureExtendedData.MipSizesInBytes.Num() == OutChildBuildInputs.TextureDescription.NumMips);
	
	{
		FSharedBuffer InputTextureMipTailData;
		if (OutChildBuildInputs.TextureDescription.NumMips > OutChildBuildInputs.NumStreamingMips)
		{
			InputTextureMipTailData = Context.FindInput(UTF8TEXTVIEW("MipTail"));
			if (!InputTextureMipTailData)
			{
				Context.AddError(TEXTVIEW("Couldn't find expected packed non-streaming mips in build"));
				return false;
			}
		}

		uint64 CurrentMipTailOffset = 0;
		for (int32 MipIndex = 0; MipIndex < OutChildBuildInputs.NumEncodedMips; MipIndex++)
		{
			FSharedBuffer MipData;
			if (MipIndex >= OutChildBuildInputs.NumStreamingMips)
			{
				// Mip tail.
				uint64 SourceMipSize = OutChildBuildInputs.TextureExtendedData.MipSizesInBytes[MipIndex];
				MipData = FSharedBuffer::MakeView(InputTextureMipTailData.GetView().Mid(CurrentMipTailOffset, SourceMipSize), InputTextureMipTailData);
				CurrentMipTailOffset += SourceMipSize;
			}
			else
			{
				TUtf8StringBuilder<10> StreamingMipName;
				StreamingMipName << "Mip" << MipIndex;
				MipData = Context.FindInput(StreamingMipName);
			}

			if (MipData.GetSize() != OutChildBuildInputs.TextureExtendedData.MipSizesInBytes[MipIndex])
			{
				TStringBuilder<256> Error;
				Error.Appendf(TEXT("Unexpected mip size when unpacking parent build: got %d, expected %d"), MipData.GetSize(), OutChildBuildInputs.TextureExtendedData.MipSizesInBytes[MipIndex]);
				Context.AddError(Error);
				return false;
			}
			OutChildBuildInputs.MipBuffers.Add(MipData);
		}
	}

	OutChildBuildInputs.CPUCopyImageInfo = FCompositeBuffer(Context.FindInput(UTF8TEXTVIEW("CPUCopyImageInfo")));
	OutChildBuildInputs.CPUCopyRawData = Context.FindInput(UTF8TEXTVIEW("CPUCopyRawData"));

	return true;
}

static void WriteChildBuildOutputs(UE::DerivedData::FBuildContext& Context, FChildBuildData&& BuildOutputs)
{
	for (int32 MipIndex = 0; MipIndex < BuildOutputs.NumStreamingMips; MipIndex++)
	{
		TUtf8StringBuilder<10> StreamingMipName;
		StreamingMipName << "Mip" << MipIndex;
		Context.AddValue(UE::DerivedData::FValueId::FromName(StreamingMipName), MoveTemp(BuildOutputs.MipBuffers[MipIndex]));
	}

	//
	// The actual streaming mips for the build might be different based on packed mip tails... however in order
	// to facilitate input/output connection between build jobs we want to always emit the full set of streaming mips
	// as outputs even if they are empty.
	//
	if (BuildOutputs.TextureExtendedData.NumMipsInTail)
	{
		int32 UnadjustedNumStreamingMips = BuildOutputs.TextureDescription.GetNumStreamingMips(nullptr, BuildOutputs.EngineParameters);
		if (UnadjustedNumStreamingMips != BuildOutputs.NumStreamingMips)
		{
			FSharedBuffer EmptyBuffer = FUniqueBuffer::Alloc(0).MoveToShared();
			for (int32 EmptyStreamingMipIndex = BuildOutputs.NumStreamingMips; EmptyStreamingMipIndex < UnadjustedNumStreamingMips; EmptyStreamingMipIndex++)
			{
				TUtf8StringBuilder<10> StreamingMipName;
				StreamingMipName << "Mip" << EmptyStreamingMipIndex;
				Context.AddValue(UE::DerivedData::FValueId::FromName(StreamingMipName), EmptyBuffer);
			}
		}
	}

	if (BuildOutputs.NumStreamingMips != BuildOutputs.NumEncodedMips)
	{
		// we need to pass the non streaming mips all packed together, and we can't append composite buffers (?)
		TArray<FSharedBuffer> NonStreamingMips;
		NonStreamingMips.Reserve(BuildOutputs.NumEncodedMips - BuildOutputs.NumStreamingMips);
		for (int32 MipIndex = BuildOutputs.NumStreamingMips; MipIndex < BuildOutputs.NumEncodedMips; MipIndex++)
		{
			NonStreamingMips.Add(BuildOutputs.MipBuffers[MipIndex]);
		}
		Context.AddValue(UE::DerivedData::FValueId::FromName(UTF8TEXTVIEW("MipTail")), FCompositeBuffer(MoveTemp(NonStreamingMips)));
	}

	Context.AddValue(UE::DerivedData::FValueId::FromName(UTF8TEXTVIEW("EncodedTextureDescription")), UE::TextureBuildUtilities::EncodedTextureDescription::ToCompactBinary(BuildOutputs.TextureDescription));
	Context.AddValue(UE::DerivedData::FValueId::FromName(UTF8TEXTVIEW("EncodedTextureExtendedData")), UE::TextureBuildUtilities::EncodedTextureExtendedData::ToCompactBinary(BuildOutputs.TextureExtendedData));

	Context.AddValue(UE::DerivedData::FValueId::FromName(UTF8TEXTVIEW("CPUCopyImageInfo")), BuildOutputs.CPUCopyImageInfo);
	Context.AddValue(UE::DerivedData::FValueId::FromName(UTF8TEXTVIEW("CPUCopyRawData")), BuildOutputs.CPUCopyRawData);
}


void FTextureBuildFunction::Build(UE::DerivedData::FBuildContext& Context) const
{
	const FCbObject Settings = Context.FindConstant(UTF8TEXTVIEW("Settings"));
	if (!Settings)
	{
		UE_LOGF(LogTextureBuildFunction, Error, "Settings are not available.");
		return;
	}

	const ITextureFormat* TextureFormat = nullptr;
	FTextureBuildSettings BuildSettings;
	if (!GetResolvedBuildSettings(Settings, &BuildSettings, &TextureFormat))
	{
		return;
	}
	
	TArray<FImage> SourceMips;
	if (!TryReadTextureSourceFromCompactBinary(Settings["Source"], Context, BuildSettings.bVolume, SourceMips))
	{
		return;
	}

	FSharedImageRef CPUCopy;
	if (BuildSettings.bCPUAccessible)
	{
		CPUCopy = new FSharedImage();
		SourceMips[0].CopyTo(*CPUCopy);
	
		// We just use a placeholder texture rather than the source.
		SourceMips.Empty();
		FImage& Placeholder = SourceMips.AddDefaulted_GetRef();
		UE::TextureBuildUtilities::GetPlaceholderTextureImage(&Placeholder);
	}

	TArray<FImage> AssociatedNormalSourceMips;
	if (FCbFieldView CompositeSource = Settings["CompositeSource"];
		CompositeSource && !TryReadTextureSourceFromCompactBinary(CompositeSource, Context, BuildSettings.bVolume, AssociatedNormalSourceMips))
	{
		return;
	}

	// SourceMips will be cleared by BuildTexture.  Store info from it for use later.
	const int32 SourceMipsNum = SourceMips.Num();
	const int32 SourceMipsNumSlices = SourceMips[0].NumSlices;
	const int32 SourceMip0SizeX = SourceMips[0].SizeX;
	const int32 SourceMip0SizeY = SourceMips[0].SizeY;
	bool bHasCompositeSource = AssociatedNormalSourceMips.Num() > 0;

	// @todo Oodle : Context.GetName() is the "build.action" file name, we want the Texture name
	//		(we want to log *both* not one or the other)

	UE_LOGF(LogTextureBuildFunction, Display, "Compressing [%ls] from %dx%d (%d slices, %d mips) to %ls...%ls%ls%ls%ls%ls RequiredMemory=%.3f MB", 
		*Context.GetName(), 
		SourceMip0SizeX, SourceMip0SizeY, SourceMipsNumSlices, SourceMipsNum,
		*BuildSettings.TextureFormatName.ToString(),
		bHasCompositeSource ? TEXT(" Composite") : TEXT(""),
		BuildSettings.bVolume ? TEXT(" Volume") : TEXT(""),
		BuildSettings.bCubemap ? TEXT(" Cube") : TEXT(""),
		BuildSettings.bLongLatSource ? TEXT(" LongLat") : TEXT(""),
		BuildSettings.bTextureArray ? TEXT(" Array") : TEXT(""),
		Context.GetRequiredMemory()/(1024.0*1024)
		);

	ITextureCompressorModule& TextureCompressorModule = FModuleManager::GetModuleChecked<ITextureCompressorModule>(TEXTURE_COMPRESSOR_MODULENAME);
	
	bool DoMemoryCheck = false;
	
#if !(WITH_EDITOR) // is standalone TBW
	// -tbfmemcheck -ansimalloc
	if ( FParse::Param(FCommandLine::Get(), TEXT("tbfmemcheck")) )
	{
		DoMemoryCheck = true;
		if ( ! FParse::Param(FCommandLine::Get(), TEXT("ansimalloc")) )
		{
			UE_LOGF(LogTextureBuildFunction, Display, "NOTE: Memory use report may be inaccurate; use -ansimalloc.");
		}
	}
#endif

	if ( DoMemoryCheck )
	{
		// do an encode of a tiny 4x4 image first, with same settings
		// this runs through the code once, and allocates some of the globals that are init-on-first-use that will stick around

		TArray<FImage> FakeSourceMips;
		FakeSourceMips.SetNum(1);
		FImageCore::ResizeImageAllocDest(SourceMips[0],FakeSourceMips[0],4,4);
		
		TArray<FImage> FakeAssociatedNormalSourceMips;
		if ( bHasCompositeSource )
		{
			FakeAssociatedNormalSourceMips = FakeSourceMips;
		}

		TArray<FCompressedImage2D> FakeCompressedMips;
		uint32 FakeNumMipsInTail;
		uint32 FakeExtData;
		UE::TextureBuildUtilities::FTextureBuildMetadata FakeBuildMetadata;

		TextureCompressorModule.BuildTexture(
			FakeSourceMips,
			FakeAssociatedNormalSourceMips,
			BuildSettings,
			Context.GetName(),
			FakeCompressedMips,
			FakeNumMipsInTail,
			FakeExtData,
			&FakeBuildMetadata
			);
	}
	
	FPlatformMemoryStats MemStatsBefore = FPlatformMemory::GetStats();

	// note: getting Metadata here means ComputeMipChainHash is called, unlike in DDC1 use

	TArray<FCompressedImage2D> CompressedMips;
	uint32 NumMipsInTail;
	uint32 ExtData;
	UE::TextureBuildUtilities::FTextureBuildMetadata BuildMetadata;
	bool bBuildSucceeded = TextureCompressorModule.BuildTexture(
		SourceMips,
		AssociatedNormalSourceMips,
		BuildSettings,
		Context.GetName(),
		CompressedMips,
		NumMipsInTail,
		ExtData,
		&BuildMetadata
		);
	if (!bBuildSucceeded)
	{
		return;
	}
	check(CompressedMips.Num() > 0);
	// SourceMips may have been freed by BuildTexture, do not use them any more
	SourceMips.Reset();
	
	uint64 BuildMemAllocated = 0;
	
	if ( DoMemoryCheck )
	{
		// if DoMemoryCheck is off, you could still do this scope to get BuildMemAllocated
		//	but it would not be accurate, so it would be misleading, so just don't do it

		FPlatformMemoryStats MemStatsAfter = FPlatformMemory::GetStats();

		if ( MemStatsAfter.PeakUsedVirtual == MemStatsBefore.PeakUsedVirtual )
		{
			// peak did not occur during BuildTexture
			//	(it occurred in startup/init)
			//	so we do not have a useful reading
			BuildMemAllocated = 0;
		}
		else
		{
			// take Peak observed during Build and subtract Pagefile before (not Peak before)
			BuildMemAllocated = MemStatsAfter.PeakUsedVirtual - MemStatsBefore.UsedVirtual;
		}
	}

	// log built info :
	{
		int32 CompressedMipCount = CompressedMips.Num();

		int64 CompressedDataSizeTotal = 0;
		for( const FCompressedImage2D & CompressedMip : CompressedMips )
		{
			CompressedDataSizeTotal += CompressedMip.RawData.Num();
		}
		
		const FCompressedImage2D & CompressedImage = CompressedMips[0];

		// log what the TextureFormat built :
		UE_LOGF(LogTextureBuildFunction, Display, "Built texture: %d Mips PF=%d=%ls : %dx%dx%d : CompressedDataSize=%lld , MemAllocated = %.3f MB",
			//[%.*s] DebugTexturePathName.Len(),DebugTexturePathName.GetData(),
			CompressedMipCount, (int)CompressedImage.PixelFormat,
			GetPixelFormatString((EPixelFormat)CompressedImage.PixelFormat),
			CompressedImage.SizeX, CompressedImage.SizeY, CompressedImage.NumSlicesWithDepth,
			CompressedDataSizeTotal,
			BuildMemAllocated/(1024.0*1024));

		// log csv line
		
		FString CSVFilename;
		if ( FParse::Value(FCommandLine::Get(), TEXT("tbfcsv="),CSVFilename) ||
			FParse::Param(FCommandLine::Get(), TEXT("tbfcsv")) )
		{
			if ( CSVFilename.IsEmpty() || CSVFilename[0] == TEXT('-') )
			{
				CSVFilename = TEXT("tbf.csv");
			}

			TUniquePtr<FArchive> OutputArchive(IFileManager::Get().CreateFileWriter(*CSVFilename, FILEWRITE_Append));
			if (!OutputArchive.IsValid())
			{
				UE_LOGF(LogTextureBuildFunction, Display, "Failed to save CSV file %ls", *CSVFilename);
			}
			else
			{
				OutputArchive->Logf( TEXT("%s,%d,%d,%d,%d,%lld,%s,%s,%s%s%s%s%s,%d,%d,%d,%lld,%.3f,%.3f"),
					*Context.GetName(), // @todo : we want texture name and the build.action file name both
					SourceMip0SizeX, SourceMip0SizeY, SourceMipsNumSlices, SourceMipsNum,
					(int64)SourceMip0SizeX*SourceMip0SizeY*SourceMipsNumSlices,
					*BuildSettings.TextureFormatName.ToString(),
					GetPixelFormatString((EPixelFormat)CompressedImage.PixelFormat),
					bHasCompositeSource ? TEXT(" Composite") : TEXT(""),
					BuildSettings.bVolume ? TEXT(" Volume") : TEXT(""),
					BuildSettings.bCubemap ? TEXT(" Cube") : TEXT(""),
					BuildSettings.bLongLatSource ? TEXT(" LongLat") : TEXT(""),
					BuildSettings.bTextureArray ? TEXT(" Array") : TEXT(""),
			
					CompressedImage.SizeX, CompressedImage.SizeY, CompressedImage.NumSlicesWithDepth,
					CompressedDataSizeTotal,
			
					Context.GetRequiredMemory()/(1024.0*1024),
					BuildMemAllocated/(1024.0*1024)
					);

				OutputArchive->Flush();
			}
		}
	}

	if ( DoMemoryCheck )
	{
		// add a little wiggle room due to inaccuracy of measurement
		//	(eg. malloc free lists can hold this much memory, various statics and global lists)
		uint64 RequiredMemPadded = Context.GetRequiredMemory() + 1024*1024;
		
		if ( BuildMemAllocated > RequiredMemPadded )
		{
			UE_LOGF(LogTextureBuildFunction, Warning, "BuildMemAllocated (%lld) > RequiredMemPadded (%lld)",
				BuildMemAllocated,RequiredMemPadded);
		}

		// for testing, get a hard stop if we used more memory than the estimate :
		//check( BuildMemAllocated <= RequiredMemPadded );
	}

	FChildBuildData OutputData;
	{
		FTextureEngineParameters EngineParameters;
		if (UE::TextureBuildUtilities::TextureEngineParameters::FromCompactBinary(EngineParameters, Context.FindConstant(UTF8TEXTVIEW("EngineParameters"))) == false)
		{
			UE_LOGF(LogTextureBuildFunction, Error, "Engine parameters are not available.");
			return;
		}
		OutputData.EngineParameters = EngineParameters;
	}

	{
		FEncodedTextureDescription TextureDescription;
		int32 CalculatedMip0SizeX = 0, CalculatedMip0SizeY = 0, CalculatedMip0NumSlices = 0;
		int32 CalculatedMipCount = TextureCompressorModule.GetMipCountForBuildSettings(SourceMip0SizeX, SourceMip0SizeY, SourceMipsNumSlices, SourceMipsNum, BuildSettings, CalculatedMip0SizeX, CalculatedMip0SizeY, CalculatedMip0NumSlices);
		BuildSettings.GetEncodedTextureDescriptionWithPixelFormat(&TextureDescription, (EPixelFormat)CompressedMips[0].PixelFormat, CalculatedMip0SizeX, CalculatedMip0SizeY, CalculatedMip0NumSlices, CalculatedMipCount);
		OutputData.TextureDescription = MoveTemp(TextureDescription);
	}
	
	

	// ExtendedData is only really useful for textures that have a post build step for tiling,
	// however it's possible that we ran the old build process where the tiling occurs as part
	// of the BuildTexture->CompressImage step via child texture formats. In that case, we've already
	// tiled and we need to pass the data back out. Otherwise, this gets ignored and the tiling step
	// regenerates it.
	{
		FEncodedTextureExtendedData ExtendedData;
		ExtendedData.NumMipsInTail = NumMipsInTail;
		ExtendedData.ExtData = ExtData;

		OutputData.NumEncodedMips = OutputData.TextureDescription.GetNumEncodedMips(&ExtendedData);
		ExtendedData.MipSizesInBytes.AddUninitialized(OutputData.NumEncodedMips);
		for (int32 MipIndex = 0; MipIndex < OutputData.NumEncodedMips; MipIndex++)
		{
			ExtendedData.MipSizesInBytes[MipIndex] = CompressedMips[MipIndex].RawData.Num();

			OutputData.MipBuffers.Add(MakeSharedBufferFromArray(MoveTemp(CompressedMips[MipIndex].RawData)));
		}

		OutputData.TextureExtendedData = MoveTemp(ExtendedData);
	}
		
	
	OutputData.NumStreamingMips = OutputData.TextureDescription.GetNumStreamingMips(&OutputData.TextureExtendedData, OutputData.EngineParameters);

	{
		if (CPUCopy.IsValid())
		{
			FCbObject ImageInfoMetadata;
			CPUCopy->ImageInfoToCompactBinary(ImageInfoMetadata);
			OutputData.CPUCopyImageInfo = ImageInfoMetadata.GetBuffer();

			OutputData.CPUCopyRawData = MakeSharedBufferFromArray(MoveTemp(CPUCopy->RawData));
		}

		// This will get added to the build metadata in a later cl.
		// Context.AddValue(UE::DerivedData::FValueId::FromName(ANSITEXTVIEW("TextureBuildMetadata")), BuildMetadata.ToCompactBinaryWithDefaults());
		WriteChildBuildOutputs(Context, MoveTemp(OutputData));
	}
}

void GenericTextureTilingBuildFunction(UE::DerivedData::FBuildContext& Context, const ITextureTiler* Tiler, const UE::FUtf8SharedString& BuildFunctionName)
{
	FChildBuildData ChildBuildData;
	if (!ReadChildBuildInputs(ChildBuildData, Context))
	{
		TStringBuilder<256> Error;
		Error.Appendf(TEXT("Failed to read child build inputs for tiling texture %s, build function %s."), *Context.GetName(), StringCast<TCHAR>(*BuildFunctionName).Get());
		Context.AddError(Error.ToView());
		return;
	}

	// The linear build wrote out an extended data but it must be a linear extended data - convert to what we need.
	FCbObject LODBiasCb = Context.FindConstant(UTF8TEXTVIEW("LODBias"));
	ChildBuildData.TextureExtendedData = Tiler->GetExtendedDataForTexture(ChildBuildData.TextureDescription, LODBiasCb["LODBias"].AsInt8());
	ChildBuildData.NumEncodedMips = ChildBuildData.TextureDescription.GetNumEncodedMips(&ChildBuildData.TextureExtendedData);
	ChildBuildData.NumStreamingMips = ChildBuildData.TextureDescription.GetNumStreamingMips(&ChildBuildData.TextureExtendedData, ChildBuildData.EngineParameters);

	UE_LOGF(LogTextureBuildFunction, Display, "Tiling %ls with %ls -> %d source mip(s) with a tail of %d...", 
		*Context.GetName(), StringCast<TCHAR>(*BuildFunctionName).Get(), ChildBuildData.TextureDescription.NumMips, ChildBuildData.TextureExtendedData.NumMipsInTail);

	//
	// Careful - the linear build might have a different streaming mip count than we output due to mip tail
	// packing.
	//

	// If the platform packs mip tails, we need to pass all the relevant mip buffers at once.
	int32 FirstMipTailIndex;
	int32 MipTailCount;
	ChildBuildData.TextureDescription.GetEncodedMipIterators(&ChildBuildData.TextureExtendedData, FirstMipTailIndex, MipTailCount);

	// We pass views to the tiler, maybe should change,
	TArray<FMemoryView, TInlineAllocator<FEncodedTextureExtendedData::MAX_TEXTURE_MIP_COUNT>> MipViews;
	for (FSharedBuffer& MipBuffer : ChildBuildData.MipBuffers)
	{
		MipViews.Add(MipBuffer.GetView());
	}

	// Process the mips
	for (int32 MipIndex = 0; MipIndex < FirstMipTailIndex + 1; MipIndex++)
	{
		int32 MipsRepresentedThisIndex = MipIndex == FirstMipTailIndex ? MipTailCount : 1;

		TArrayView<FMemoryView> MipsThisIndex = MakeArrayView(MipViews.GetData() + MipIndex, MipsRepresentedThisIndex);

		FSharedBuffer MipData = Tiler->ProcessMipLevel(ChildBuildData.TextureDescription, ChildBuildData.TextureExtendedData, MipsThisIndex, MipIndex);

		// Make sure we got the size we advertised prior to the build. If this ever fires then we
		// have a critical mismatch!
		check(ChildBuildData.TextureExtendedData.MipSizesInBytes[MipIndex] == MipData.GetSize());

		ChildBuildData.MipBuffers[MipIndex] = MoveTemp(MipData);
	} // end for each mip

	WriteChildBuildOutputs(Context, MoveTemp(ChildBuildData));
}

void GenericTextureDecodeBuildFunction(UE::DerivedData::FBuildContext& Context, const UE::FUtf8SharedString& BuildFunctionName)
{
	FChildBuildData ChildBuildInputs;
	if (!ReadChildBuildInputs(ChildBuildInputs, Context))
	{
		TStringBuilder<256> Error;
		Error.Appendf(TEXT("Failed to read child build inputs for decoding texture %s, build function %s."), *Context.GetName(), StringCast<TCHAR>(*BuildFunctionName).Get());
		Context.AddError(Error.ToView());
		return;
	}

	// Read inputs unique to us.
	FName BaseTextureFormatName = NAME_None;
	bool bSRGB = false;
	const ITextureFormat* BaseTextureFormat = nullptr;
	{
		FCbObject TextureInfoCb = Context.FindConstant(UTF8TEXTVIEW("TextureInfo"));
		ReadCbField(TextureInfoCb["BaseFormatName"], BaseTextureFormatName);
		uint16 RequiredVersion = TextureInfoCb["BaseFormatVersion"].AsUInt16();
		bSRGB = TextureInfoCb["bSRGB"].AsBool();

		if (ITextureFormatManagerModule* TFM = GetTextureFormatManager())
		{
			BaseTextureFormat = TFM->FindTextureFormat(BaseTextureFormatName);
		}

		if (!BaseTextureFormat)
		{
			TStringBuilder<256> Error;
			Error << TEXT("Missing texture format: ") << BaseTextureFormatName;
			Context.AddError(Error.ToView());
			return;
		}

		uint16 OurVersion = BaseTextureFormat->GetVersion(BaseTextureFormatName);
		if (OurVersion != RequiredVersion)
		{
			TStringBuilder<256> Error;
			Error.Appendf(TEXT("%s has version %hu when version %hu is required."), *BaseTextureFormatName.ToString(), OurVersion, RequiredVersion);
			Context.AddError(Error.ToView());
			return;
		}
	}

	UE_LOGF(LogTextureBuildFunction, Display, "Decoding %ls with %ls...", *Context.GetName(), StringCast<TCHAR>(*BuildFunctionName).Get());


	if (!BaseTextureFormat->CanDecodeFormat(ChildBuildInputs.TextureDescription.PixelFormat))
	{
		TStringBuilder<256> Error;
		Error.Appendf(TEXT("Texture format %s can't decode image format %s"), *BaseTextureFormatName.ToString(), GetPixelFormatString(ChildBuildInputs.TextureDescription.PixelFormat));
		Context.AddError(Error);
		return;
	}

	EPixelFormat DecodedPixelFormat = PF_Unknown;
	for (int32 MipIndex = 0; MipIndex < ChildBuildInputs.NumEncodedMips; MipIndex++)
	{
		int32 NumSlicesWithDepth = ChildBuildInputs.TextureDescription.GetNumSlices_WithDepth(MipIndex);
		int32 SizeX = ChildBuildInputs.TextureDescription.GetMipWidth(MipIndex);
		int32 SizeY = ChildBuildInputs.TextureDescription.GetMipHeight(MipIndex);

		FImage DecodedImage;
		if (!BaseTextureFormat->DecodeImage(SizeX, SizeY, NumSlicesWithDepth, ChildBuildInputs.TextureDescription.PixelFormat, 
			bSRGB, BaseTextureFormatName, ChildBuildInputs.MipBuffers[MipIndex], DecodedImage, Context.GetName()))
		{
			TStringBuilder<256> Error;
			Error.Appendf(TEXT("Texture format %s failed to decode image format %s, mip %d"), *BaseTextureFormatName.ToString(), GetPixelFormatString(ChildBuildInputs.TextureDescription.PixelFormat), MipIndex);
			Context.AddError(Error);
			return;
		}

		ERawImageFormat::Type NeededConversion;
		DecodedPixelFormat = FImageCoreUtils::GetPixelFormatForRawImageFormat(DecodedImage.Format, &NeededConversion);
		if (NeededConversion != DecodedImage.Format)
		{
			FImage ConvertedImage;
			DecodedImage.CopyTo(ConvertedImage, NeededConversion, DecodedImage.GammaSpace);
			ChildBuildInputs.MipBuffers[MipIndex] = MakeSharedBufferFromArray(MoveTemp(ConvertedImage.RawData));
		}
		else
		{
			ChildBuildInputs.MipBuffers[MipIndex] = MakeSharedBufferFromArray(MoveTemp(DecodedImage.RawData));
		}

		ChildBuildInputs.TextureExtendedData.MipSizesInBytes[MipIndex] = ChildBuildInputs.MipBuffers[MipIndex].GetSize();
	}

	ChildBuildInputs.TextureDescription.PixelFormat = DecodedPixelFormat;

	WriteChildBuildOutputs(Context, MoveTemp(ChildBuildInputs));
}

void GenericTextureDetileBuildFunction(UE::DerivedData::FBuildContext& Context, const ITextureTiler* Tiler, const UE::FUtf8SharedString& BuildFunctionName)
{
	FChildBuildData ChildBuildInputs;
	if (!ReadChildBuildInputs(ChildBuildInputs, Context))
	{
		TStringBuilder<256> Error;
		Error.Appendf(TEXT("Failed to read child build inputs for detiling texture %s, build function %s."), *Context.GetName(), StringCast<TCHAR>(*BuildFunctionName).Get());
		Context.AddError(Error.ToView());
		return;
	}

	UE_LOGF(LogTextureBuildFunction, Display, "De-Tiling %ls with %ls -> %d source mip(s) with a tail of %d...", 
		*Context.GetName(), StringCast<TCHAR>(*BuildFunctionName).Get(), ChildBuildInputs.TextureDescription.NumMips, ChildBuildInputs.TextureExtendedData.NumMipsInTail);

	FEncodedTextureDescription::FUniqueBufferMipChain LinearMips;
	Tiler->DetileMipChain(LinearMips, ChildBuildInputs.MipBuffers, ChildBuildInputs.TextureDescription, ChildBuildInputs.TextureExtendedData, *Context.GetName());

	ChildBuildInputs.MipBuffers.Reset(LinearMips.Num());
	for (FUniqueBuffer& Buffer : LinearMips)
	{
		ChildBuildInputs.MipBuffers.Add(Buffer.MoveToShared());
	}

	// After we detile, we are a linear texture:
	ChildBuildInputs.TextureExtendedData = FEncodedTextureExtendedData();
	ChildBuildInputs.NumEncodedMips = ChildBuildInputs.TextureDescription.GetNumEncodedMips(nullptr);
	ChildBuildInputs.NumStreamingMips = ChildBuildInputs.TextureDescription.GetNumStreamingMips(nullptr, ChildBuildInputs.EngineParameters);

	WriteChildBuildOutputs(Context, MoveTemp(ChildBuildInputs));
}
