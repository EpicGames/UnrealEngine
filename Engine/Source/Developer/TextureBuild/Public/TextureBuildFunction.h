// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/SharedString.h"
#include "DerivedDataBuildFunction.h"
#include "DerivedDataBuildVersion.h"
#include "DerivedDataCacheKey.h"
#include "Misc/Guid.h"

class ITextureFormat;
class ITextureTiler;
namespace UE { namespace DerivedData { class FBuildVersionBuilder; } }

class FTextureBuildFunction : public UE::DerivedData::IBuildFunction
{
public:
	TEXTUREBUILD_API virtual FGuid GetVersion() const final;
	TEXTUREBUILD_API virtual void Configure(UE::DerivedData::FBuildConfigContext& Context) const override;
	TEXTUREBUILD_API virtual void Build(UE::DerivedData::FBuildContext& Context) const override;

protected:
	virtual void GetVersion(UE::DerivedData::FBuildVersionBuilder& Builder, ITextureFormat*& OutTextureFormatVersioning) const = 0;
};

/**
*	This function does the meat of breaking out the inputs from the build context and handing them
*	to the tiler, then packing them back up for the build process.
*/
TEXTUREBUILD_API void GenericTextureTilingBuildFunction(UE::DerivedData::FBuildContext& Context, const ITextureTiler* Tiler, const UE::FUtf8SharedString& BuildFunctionName);


/**
*	This build function expects an implementation of ITextureTiler as its template
*	and looks a bit awkward because IBuildFunction is required to not have any state,
*	so we can't put the instance of the ITextureTiler on our object - hence the odd
*	statics.
*/
template <class ITextureTilerObject>
class FGenericTextureTilingBuildFunction : public UE::DerivedData::IBuildFunction
{
	virtual void Build(UE::DerivedData::FBuildContext& Context) const
	{
		ITextureTilerObject Tiler;
		GenericTextureTilingBuildFunction(Context, &Tiler, GetName());
	}
	virtual void Configure(UE::DerivedData::FBuildConfigContext& Context) const
	{
		Context.SetCacheBucket(UE::DerivedData::FCacheBucket(UTF8TEXTVIEW("TiledTextures")));
	}
	virtual FGuid GetVersion() const final { return ITextureTilerObject::GetBuildFunctionVersionGuid(); }
	const UE::FUtf8SharedString& GetName() const final
	{
		static UE::FUtf8SharedString Name(ITextureTilerObject::GetBuildFunctionNameStatic());
		return Name;
	}
};

TEXTUREBUILD_API void GenericTextureDetileBuildFunction(UE::DerivedData::FBuildContext& Context, const ITextureTiler* Tiler, const UE::FUtf8SharedString& BuildFunctionName);

template <class ITextureTilerObject>
class FGenericTextureDetileBuildFunction : public UE::DerivedData::IBuildFunction
{
	virtual void Build(UE::DerivedData::FBuildContext& Context) const
	{
		ITextureTilerObject Tiler;
		GenericTextureDetileBuildFunction(Context, &Tiler, GetName());
	}
	virtual void Configure(UE::DerivedData::FBuildConfigContext& Context) const
	{
		Context.SetCacheBucket(UE::DerivedData::FCacheBucket(UTF8TEXTVIEW("DetiledTextures")));
	}
	virtual FGuid GetVersion() const final { return ITextureTilerObject::GetBuildFunctionVersionGuid(); }
	const UE::FUtf8SharedString& GetName() const final
	{
		static UE::FUtf8SharedString Name(ITextureTilerObject::GetDetileBuildFunctionNameStatic());
		return Name;
	}
};

TEXTUREBUILD_API void GenericTextureDecodeBuildFunction(UE::DerivedData::FBuildContext& Context, const UE::FUtf8SharedString& BuildFunctionName);

template <class ITextureFormatObject>
class FGenericTextureDecodeBuildFunction : public UE::DerivedData::IBuildFunction
{
	virtual void Build(UE::DerivedData::FBuildContext& Context) const
	{
		GenericTextureDecodeBuildFunction(Context, GetName());
	}
	virtual void Configure(UE::DerivedData::FBuildConfigContext& Context) const
	{
		Context.SetCacheBucket(UE::DerivedData::FCacheBucket(UTF8TEXTVIEW("DecodedTextures")));
	}
	virtual FGuid GetVersion() const final { return ITextureFormatObject::GetDecodeBuildFunctionVersionGuid(); }
	const UE::FUtf8SharedString& GetName() const final
	{
		static UE::FUtf8SharedString Name(ITextureFormatObject::GetDecodeBuildFunctionNameStatic());
		return Name;
	}
};

