// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureBuild : ModuleRules
{
	public TextureBuild(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"Engine",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"DerivedDataCache",
			"ImageCore",
			"ImageWrapper",
			"TextureBuildUtilities",
			"TextureCompressor",
			"TextureFormat",
		});
	}
}
